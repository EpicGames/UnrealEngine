{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Example Device Profile Selector",
	"Description" : "Example Device Profile Selector used show selection of device profiles on hardware",
	"Category" : "Device Profile Selectors",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "ExampleDeviceProfileSelector",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PostConfigInit"
		}
	]
}