// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class ExampleDeviceProfileSelector : ModuleRules
	{
        public ExampleDeviceProfileSelector(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
				    "Core",
				    "CoreUObject",
				    "Engine",
				}
				);
		}
	}
}
