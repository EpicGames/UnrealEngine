// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExampleDeviceProfileSelectorModule.h"
#include "Modules/ModuleManager.h"


IMPLEMENT_MODULE(FExampleDeviceProfileSelectorModule, ExampleDeviceProfileSelector);


void FExampleDeviceProfileSelectorModule::StartupModule()
{
}


void FExampleDeviceProfileSelectorModule::ShutdownModule()
{
}


const FString FExampleDeviceProfileSelectorModule::GetRuntimeDeviceProfileName()
{
	FString ProfileName = FPlatformProperties::PlatformName();
	UE_LOGF(LogInit, Log, "Selected Device Profile: [%ls]", *ProfileName);
	return ProfileName;
}
