// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDeviceProfileSelectorModule.h"

/**
 * Implements the Example Device Profile Selector module.
 */
class FExampleDeviceProfileSelectorModule
	: public IDeviceProfileSelectorModule
{
public:

	//~ Begin IDeviceProfileSelectorModule Interface
	virtual const FString GetRuntimeDeviceProfileName() override;
	//~ End IDeviceProfileSelectorModule Interface


	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

	
	/**
	 * Virtual destructor.
	 */
	virtual ~FExampleDeviceProfileSelectorModule()
	{
	}
};
